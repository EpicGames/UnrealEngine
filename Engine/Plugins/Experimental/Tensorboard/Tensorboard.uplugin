{
    "FileVersion": 3,
    "Version": 1,
    "VersionName": "0.1",
    "FriendlyName": "",
    "Description": "Optional tensorboard python package and dependencies",
    "Category": "Scripting",
    "CreatedBy": "Epic Games, Inc.",
    "CreatedByURL": "https://epicgames.com",
    "DocsURL": "",
    "MarketplaceURL": "",
    "SupportURL": "https://forums.unrealengine.com/",
    "CanContainContent": true,
    "IsBetaVersion": false,
    "IsExperimentalVersion": true,
    "Installed": false,
    "Plugins": [
        {
            "Name": "PythonMLPackages",
            "Enabled": true
        }
    ],
    "PythonRequirements": [
        {
            "Platform": "All",
            "Requirements": [
                "absl-py==2.1.0 --hash=sha256:526a04eadab8b4ee719ce68f204172ead1027549089702d99b9059f129ff1308",
                "cachetools==5.5.0 --hash=sha256:02134e8439cdc2ffb62023ce1debca2944c3f289d66bb17ead3ab3dede74b292",
                "google-auth==2.35.0 --hash=sha256:25df55f327ef021de8be50bad0dfd4a916ad0de96da86cd05661c9297723ad3f",
                "google-auth-oauthlib==1.0.0 --hash=sha256:95880ca704928c300f48194d1770cf5b1462835b6e49db61445a520f793fd5fb",
                "grpcio==1.67.0 --hash=sha256:16724ffc956ea42967f5758c2f043faef43cb7e48a51948ab593570570d1e68b --hash=sha256:3840994689cc8cbb73d60485c594424ad8adb56c71a30d8948d6453083624b52 --hash=sha256:985b2686f786f3e20326c4367eebdaed3e7aa65848260ff0c6644f817042cb15 --hash=sha256:a4e95e43447a02aa603abcc6b5e727d093d161a869c83b073f50b9390ecf0fa8 --hash=sha256:ac11ecb34a86b831239cc38245403a8de25037b448464f95c3315819e7519772 --hash=sha256:b2a44e572fb762c668e4812156b81835f7aba8a721b027e2d4bb29fb50ff4d33 --hash=sha256:b6eb68493a05d38b426604e1dc93bfc0137c4157f7ab4fac5771fd9a104bbaa6 --hash=sha256:cb204a742997277da678611a809a8409657b1398aaeebf73b3d9563b7d154c13 --hash=sha256:d90cfdafcf4b45a7a076e3e2a58e7bc3d59c698c4f6470b0bb13a4d869cf2273 --hash=sha256:eef1dce9d1a46119fd09f9a992cf6ab9d9178b696382439446ca5f399d7b96fe --hash=sha256:f55f077685f61f0fbd06ea355142b71e47e4a26d2d678b3ba27248abfe67163a --hash=sha256:fe89295219b9c9e47780a0f1c75ca44211e706d1c598242249fe717af3385ec8",
                "Markdown==3.7 --hash=sha256:7eb6df5690b81a1d7942992c97fad2938e956e79df20cbc6186e9c3a77b1c803",
                "oauthlib==3.2.2 --hash=sha256:8139f29aac13e25d502680e9e19963e83f16838d48a0d71c287fe40e7067fbca",
                "pyasn1==0.6.1 --hash=sha256:0d632f46f2ba09143da3a8afe9e33fb6f92fa2320ab7e886e2d0f7672af84629",
                "pyasn1_modules==0.4.1 --hash=sha256:49bfa96b45a292b711e986f222502c1c9a5e1f4e568fc30e2574a6c7d07838fd",
                "requests-oauthlib==2.0.0 --hash=sha256:7dd8a5c40426b779b0868c404bdef9768deccf22749cde15852df527e6269b36",
                "rsa==4.9 --hash=sha256:90260d9058e514786967344d0ef75fa8727eed8a7d2e43ce9f4bcf1b536174f7",
                "six==1.16.0 --hash=sha256:8abb2f1d86890a2dfb989f9a77cfcfd3e47c2a354b01111771326f8aa26e0254",
                "tensorboard==2.14.1 --hash=sha256:3db108fb58f023b6439880e177743c5f1e703e9eeb5fb7d597871f949f85fd58",
                "tensorboard-data-server==0.7.2 --hash=sha256:7e0610d205889588983836ec05dc098e80f97b7e7bbff7e994ebb78f578d0ddb --hash=sha256:9fe5d24221b29625dbc7328b0436ca7fc1c23de4acf4d272f1180856e32f9f60 --hash=sha256:ef687163c24185ae9754ed5650eb5bc4d84ff257aabdc33f0cc6f74d8ba54530",
                "Werkzeug==3.0.4 --hash=sha256:02c9eb92b7d6c06f31a782811505d2157837cea66aaede3e217c7c27c039476c"
            ]
        }
    ]
}